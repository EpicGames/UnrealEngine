// Copyright Epic Games, Inc. All Rights Reserved.

#include "RevisionControlOverlaySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevisionControlOverlaySettings)

FSimpleMulticastDelegate URevisionControlOverlaySettings::OnOverlayStatesChanged;
FSimpleMulticastDelegate URevisionControlOverlaySettings::OnOverlayColorsChanged;

void URevisionControlOverlaySettings::NotifyOverlayStatesChanged()
{
	OnOverlayStatesChanged.Broadcast();
}

void URevisionControlOverlaySettings::NotifyOverlayColorsChanged()
{
	OnOverlayColorsChanged.Broadcast();
}

#if WITH_EDITOR
void URevisionControlOverlaySettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(URevisionControlOverlaySettings, OverlayAlpha))
	{
		NotifyOverlayColorsChanged();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(URevisionControlOverlaySettings, bShowCheckedOutByOtherUser)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(URevisionControlOverlaySettings, bShowNotAtHeadRevision)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(URevisionControlOverlaySettings, bShowCheckedOut)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(URevisionControlOverlaySettings, bShowOpenForAdd))
	{
		NotifyOverlayStatesChanged();
	}
}
#endif
