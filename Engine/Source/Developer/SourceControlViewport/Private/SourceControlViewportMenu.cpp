// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlViewportMenu.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Widgets/Input/SSpinBox.h"
#include "EngineAnalytics.h"
#include "LevelEditorMenuContext.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SourceControlViewportMenu"

static const FName MenuName("LevelEditor.LevelViewportToolbar.Show");
static const FName SectionName("LevelViewportEditorShow");
static const FName SubMenuName("ShowRevisionControlMenu");

FSourceControlViewportMenu::FSourceControlViewportMenu()
{
}

FSourceControlViewportMenu::~FSourceControlViewportMenu()
{
	RemoveViewportMenu();
}

void FSourceControlViewportMenu::Init()
{
}

void FSourceControlViewportMenu::SetEnabled(bool bInEnabled)
{
	if (bInEnabled)
	{
		InsertViewportMenu();
	}
	else
	{
		RemoveViewportMenu();
	}
}

void FSourceControlViewportMenu::InsertViewportMenu()
{
	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName);
		if (Menu != nullptr)
		{
			Menu->AddDynamicSection(SectionName,
				FNewToolMenuDelegate::CreateSP(this, &FSourceControlViewportMenu::PopulateViewportMenu)
			);
		}
	}
}

void FSourceControlViewportMenu::PopulateViewportMenu(UToolMenu* InMenu)
{
	check(InMenu);
	
	FToolMenuSection& RevisionControlSection = InMenu->FindOrAddSection(TEXT("AllShowFlags"));
	
	RevisionControlSection.AddSubMenu(
		SubMenuName,
		LOCTEXT("RevisionControlSubMenu", "Revision Control"),
		LOCTEXT("RevisionControlSubMenu_ToolTip", "Toggle revision control viewport options on or off."),
		FNewToolMenuDelegate::CreateSP(this, &FSourceControlViewportMenu::PopulateRevisionControlMenu),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.RevisionControl")
	);
}

void FSourceControlViewportMenu::PopulateRevisionControlMenu(UToolMenu* InMenu)
{
	TSharedPtr<SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(InMenu);
	if (!Viewport)
	{
		return;
	}

	FToolMenuSection& DefaultSection = InMenu->AddSection(NAME_None, LOCTEXT("RevisionControlSectionStatus", "Status Highlights"));

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("ShowAll", "Show All"),
		LOCTEXT("ShowAll_ToolTip", "Enable highlighting for all statuses"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::ShowAll, Viewport.ToWeakPtr())
		),
		EUserInterfaceActionType::Button
	);

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("HideAll", "Hide All"),
		LOCTEXT("HideAll_ToolTip", "Disable highlighting for all statuses"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::HideAll, Viewport.ToWeakPtr())
		),
		EUserInterfaceActionType::Button
	);

	DefaultSection.AddSeparator(NAME_None);

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("HighlightCheckedOutByOtherUser", "Checked Out by Others"),
		LOCTEXT("HighlightCheckedOutByOtherUser_ToolTip", "Highlight objects that are checked out by someone else."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ShowMenu.CheckedOutByOtherUser"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::ToggleHighlight, Viewport.ToWeakPtr(), ESourceControlStatus::CheckedOutByOtherUser),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSourceControlViewportMenu::IsHighlighted, Viewport.ToWeakPtr(), ESourceControlStatus::CheckedOutByOtherUser)
		),
		EUserInterfaceActionType::ToggleButton
	);

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("HighlightNotAtHeadRevision", "Out of Date"),
		LOCTEXT("HighlightNotAtHeadRevision_ToolTip", "Highlight objects that are not at the latest revision."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ShowMenu.NotAtHeadRevision"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::ToggleHighlight, Viewport.ToWeakPtr(), ESourceControlStatus::NotAtHeadRevision),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSourceControlViewportMenu::IsHighlighted, Viewport.ToWeakPtr(), ESourceControlStatus::NotAtHeadRevision)
		),
		EUserInterfaceActionType::ToggleButton
	);

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("HighlightCheckedOut", "Checked Out by Me"),
		LOCTEXT("HighlightCheckedOut_ToolTip", "Highlight objects that are checked out by me."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ShowMenu.CheckedOut"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::ToggleHighlight, Viewport.ToWeakPtr(), ESourceControlStatus::CheckedOut),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSourceControlViewportMenu::IsHighlighted, Viewport.ToWeakPtr(), ESourceControlStatus::CheckedOut)
		),
		EUserInterfaceActionType::ToggleButton
	);

	DefaultSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("HighlightOpenForAdd", "Newly Added"),
		LOCTEXT("HighlightOpenForAdd_ToolTip", "Highlight objects that have been added by me."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ShowMenu.OpenForAdd"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSourceControlViewportMenu::ToggleHighlight, Viewport.ToWeakPtr(), ESourceControlStatus::OpenForAdd),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSourceControlViewportMenu::IsHighlighted, Viewport.ToWeakPtr(), ESourceControlStatus::OpenForAdd)
		),
		EUserInterfaceActionType::ToggleButton
	);

	TSharedRef<SSpinBox<uint8>> OpacityWidget = SNew(SSpinBox<uint8>)
		.ClearKeyboardFocusOnCommit(true)
		.OnValueChanged(this, &FSourceControlViewportMenu::SetOpacityValue, Viewport.ToWeakPtr())
		.OnValueCommitted(this, &FSourceControlViewportMenu::OnOpacityCommitted, Viewport.ToWeakPtr())
		.Value(this, &FSourceControlViewportMenu::GetOpacityValue, Viewport.ToWeakPtr())
		.MinValue(0)
		.MinSliderValue(0)
		.MaxValue(100)
		.MaxSliderValue(100);

	DefaultSection.AddEntry(FToolMenuEntry::InitWidget(
		NAME_None,
		OpacityWidget,
		LOCTEXT("Opacity", "Opacity")
		)
	);
}

void FSourceControlViewportMenu::RemoveViewportMenu()
{
	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName);
		if (Menu != nullptr)
		{
			Menu->RemoveSection(SectionName);
		}
	}
}

void FSourceControlViewportMenu::ShowAll(TWeakPtr<SLevelViewport> Viewport)
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		FLevelEditorViewportClient* ViewportClient = &Pinned->GetLevelViewportClient();
	
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::CheckedOutByOtherUser, /*bEnabled=*/true);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::NotAtHeadRevision, /*bEnabled=*/true);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::CheckedOut, /*bEnabled=*/true);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::OpenForAdd, /*bEnabled=*/true);
	
		RecordToggleEvent(TEXT("All"), /*bEnabled=*/true);
	}
}

void FSourceControlViewportMenu::HideAll(TWeakPtr<SLevelViewport> Viewport)
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		FLevelEditorViewportClient* ViewportClient = &Pinned->GetLevelViewportClient();
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::CheckedOutByOtherUser, /*bEnabled=*/false);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::NotAtHeadRevision, /*bEnabled=*/false);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::CheckedOut, /*bEnabled=*/false);
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, ESourceControlStatus::OpenForAdd, /*bEnabled=*/false);

		RecordToggleEvent(TEXT("All"), /*bEnabled=*/false);
	}
}

void FSourceControlViewportMenu::ToggleHighlight(TWeakPtr<SLevelViewport> Viewport, ESourceControlStatus Status)
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		FLevelEditorViewportClient* ViewportClient = &Pinned->GetLevelViewportClient();
		bool bOld = SourceControlViewportUtils::GetFeedbackEnabled(ViewportClient, Status);
		bool bNew = bOld ? false : true;
		SourceControlViewportUtils::SetFeedbackEnabled(ViewportClient, Status, bNew);

		FString EnumValueWithoutType = UEnum::GetValueAsString(Status)
			.Replace(TEXT("ESourceControlStatus::"), TEXT(""));
		RecordToggleEvent(EnumValueWithoutType, bNew);
	}
}

bool FSourceControlViewportMenu::IsHighlighted(TWeakPtr<SLevelViewport> Viewport, ESourceControlStatus Status) const
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		return SourceControlViewportUtils::GetFeedbackEnabled(&Pinned->GetLevelViewportClient(), Status);	
	}
	return false;
}

void FSourceControlViewportMenu::OnOpacityCommitted(uint8 NewValue, ETextCommit::Type CommitType, TWeakPtr<SLevelViewport> Viewport)
{
	SetOpacityValue(NewValue, Viewport);
	SourceControlViewportUtils::SaveFeedbackSettings();
}

void FSourceControlViewportMenu::SetOpacityValue(uint8 NewValue, TWeakPtr<SLevelViewport> Viewport)
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		SourceControlViewportUtils::SetFeedbackOpacity(&Pinned->GetLevelViewportClient(), NewValue);
	}
}

uint8 FSourceControlViewportMenu::GetOpacityValue(TWeakPtr<SLevelViewport> Viewport) const
{
	if (TSharedPtr<SLevelViewport> Pinned = Viewport.Pin())
	{
		return SourceControlViewportUtils::GetFeedbackOpacity(&Pinned->GetLevelViewportClient());
	}
	return 0;
}

void FSourceControlViewportMenu::RecordToggleEvent(const FString& Param, bool bEnabled) const
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(
			TEXT("Editor.Usage.SourceControl.Settings"), Param, bEnabled ? TEXT("True") : TEXT("False")
		);
	}
}

#undef LOCTEXT_NAMESPACE