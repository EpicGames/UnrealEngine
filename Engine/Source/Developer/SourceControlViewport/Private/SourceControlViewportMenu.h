// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SourceControlViewportUtils.h"
#include "Types/SlateEnums.h"

class SLevelViewport;
class FLevelEditorViewportClient;
class UToolMenu;
class SWidget;

// Adds an options menu to the Viewport's SHOW pill.
class FSourceControlViewportMenu : public TSharedFromThis<FSourceControlViewportMenu, ESPMode::ThreadSafe>
{
public:
	FSourceControlViewportMenu();
	~FSourceControlViewportMenu();

public:
	void Init();
	void SetEnabled(bool bInEnabled);

private:
	void InsertViewportMenu();
	void PopulateViewportMenu(UToolMenu* InMenu);
	void PopulateRevisionControlMenu(UToolMenu* InMenu);
	void RemoveViewportMenu();

private:
	void ShowAll(TWeakPtr<SLevelViewport> Viewport);
	void HideAll(TWeakPtr<SLevelViewport> Viewport);

	void ToggleHighlight(TWeakPtr<SLevelViewport> Viewport, ESourceControlStatus Status);
	bool IsHighlighted(TWeakPtr<SLevelViewport> Viewport, ESourceControlStatus Status) const;

	void OnOpacityCommitted(uint8 NewValue, ETextCommit::Type CommitType, TWeakPtr<SLevelViewport> Viewport);
	void SetOpacityValue(uint8 NewValue, TWeakPtr<SLevelViewport> Viewport);
	uint8 GetOpacityValue(TWeakPtr<SLevelViewport> Viewport) const;

private:
	void RecordToggleEvent(const FString& Param, bool bEnabled) const;
};