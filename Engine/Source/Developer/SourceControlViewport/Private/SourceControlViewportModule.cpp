// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlViewportModule.h"
#include "SourceControlViewportMenu.h"
#include "SourceControlViewportToolTips.h"
#include "HAL/IConsoleManager.h"

TAutoConsoleVariable<bool> CVarSourceControlEnableViewportStatus(
	TEXT("SourceControl.ViewportStatus.Enable"),
	false,
	TEXT("Enables source control viewport status features."),
	ECVF_Default);

void FSourceControlViewportModule::StartupModule()
{
	ViewportMenu = MakeShared<FSourceControlViewportMenu>();
	ViewportMenu->Init();
	ViewportToolTips = MakeShared<FSourceControlViewportToolTips>();
	ViewportToolTips->Init();

	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.ViewportStatus.Enable")); ensure(CVar))
	{
		CVar->OnChangedDelegate().AddRaw(this, &FSourceControlViewportModule::HandleCVarChanged);
	}

	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.RevisionControl.AutoPopulateState")); ensure(CVar))
	{
		CVar->OnChangedDelegate().AddRaw(this, &FSourceControlViewportModule::HandleCVarChanged);
	}

	UpdateSettings();
}

void FSourceControlViewportModule::ShutdownModule()
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.RevisionControl.AutoPopulateState")); ensure(CVar))
	{
		CVar->OnChangedDelegate().RemoveAll(this);
	}

	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.ViewportStatus.Enable")); ensure(CVar))
	{
		CVar->OnChangedDelegate().RemoveAll(this);
	}

	ViewportToolTips.Reset();
	ViewportMenu.Reset();
}

void FSourceControlViewportModule::HandleCVarChanged(IConsoleVariable* Variable)
{
	UpdateSettings();
}

void FSourceControlViewportModule::UpdateSettings()
{
	// Are both CVar's enabled?
	bool bEnabled = true;

	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.ViewportStatus.Enable")); ensure(CVar))
	{
		bEnabled &= CVar->GetBool();
	}
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.RevisionControl.AutoPopulateState")); ensure(CVar))
	{
		bEnabled &= CVar->GetBool();
	}

	// Propagate resulting value.
	ViewportMenu->SetEnabled(bEnabled);
	ViewportToolTips->SetEnabled(bEnabled);

	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("RevisionControl.Overlays.Enable")); ensure(CVar))
	{
		CVar->Set(bEnabled);
	}
}

IMPLEMENT_MODULE( FSourceControlViewportModule, SourceControlViewport );