// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FSourceControlViewportMenu;
class FSourceControlViewportToolTips;

class FSourceControlViewportModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Gets a reference to the source control viewport module instance.
	 *
	 * @return A reference to the source control viewport module.
	 */
	static FSourceControlViewportModule& Get()
	{
		static FName SourceControlViewportModule("SourceControlViewport");
		return FModuleManager::LoadModuleChecked<FSourceControlViewportModule>(SourceControlViewportModule);
	}

	static FSourceControlViewportModule* TryGet()
	{
		static FName SourceControlViewportModule("SourceControlViewport");
		return FModuleManager::GetModulePtr<FSourceControlViewportModule>(SourceControlViewportModule);
	}

private:
	void HandleCVarChanged(IConsoleVariable* Variable);
	void UpdateSettings();

private:
	TSharedPtr<FSourceControlViewportMenu> ViewportMenu;
	TSharedPtr<FSourceControlViewportToolTips> ViewportToolTips;
};
