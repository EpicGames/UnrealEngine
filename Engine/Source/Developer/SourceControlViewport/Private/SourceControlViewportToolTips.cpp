// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlViewportToolTips.h"
#include "SourceControlViewportUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SCanvas.h"
#include "Fonts/FontMeasure.h"
#include "EngineUtils.h"
#include "LevelEditor.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SLevelViewport.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "SourceControlViewportToolTips"

FSourceControlViewportToolTips::FSourceControlViewportToolTips()
{
}

FSourceControlViewportToolTips::~FSourceControlViewportToolTips()
{
	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();
}

void FSourceControlViewportToolTips::Init()
{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateRaw(this, &FSourceControlViewportToolTips::Tick)
	);
}

void FSourceControlViewportToolTips::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

bool FSourceControlViewportToolTips::Tick(float DeltaTime)
{
	UpdateCanvas(DeltaTime);
	return true;
}

void FSourceControlViewportToolTips::UpdateCanvas(float DeltaTime)
{
	if (CanvasWidget.IsValid())
	{
		// Remove the canvas if the active viewport changed.
		if (GCurrentLevelEditingViewportClient != nullptr)
		{
			TSharedPtr<SLevelViewport> LevelViewportWidget = StaticCastSharedPtr<SLevelViewport>(GCurrentLevelEditingViewportClient->GetEditorViewportWidget());
			if (LevelViewportWidget != ViewportWidget)
			{
				RemoveCanvas();
			}
		}
	}

	if (CanvasWidget.IsValid() && !bEnabled)
	{
		// Remove the canvas if the tooltips got disabled.
		RemoveCanvas();
	}

	if (!CanvasWidget.IsValid() && bEnabled)
	{
		// Insert the canvas if the active viewport is perspective.
		InsertCanvas();
	}

	if (CanvasWidget.IsValid() && ToolTipWidget.IsValid() && ViewportWidget.IsValid())
	{
		if (TSharedPtr<SLevelViewport> ViewportWidgetPtr = ViewportWidget.Pin())
		{
			FViewport* Viewport = ViewportWidgetPtr->GetViewportClient()->Viewport;
			if (Viewport && Viewport->GetRenderTargetTexture())
			{
				int32 MouseX = Viewport->GetMouseX();
				int32 MouseY = Viewport->GetMouseY();
				if (ActorMouseX != MouseX || ActorMouseY != MouseY)
				{
					Actor.Reset();
					UpdateToolTip();
					DelayTime = 0;

					ActorMouseX = MouseX;
					ActorMouseY = MouseY;
				}
				else
				{
					DelayTime += DeltaTime;
				}

				if (DelayTime >= 0.15f) // See: Slate.TooltipSummonDelay
				{
					HActor* ActorHitProxy = HitProxyCast<HActor>(Viewport->GetHitProxy(MouseX, MouseY));
					if (ActorHitProxy == nullptr)
					{
						Actor.Reset();
						UpdateToolTip();
					}
					if (ActorHitProxy != nullptr && ActorHitProxy->Actor != Actor)
					{
						Actor.Reset();
						UpdateToolTip();

						Actor = MakeWeakObjectPtr(ActorHitProxy->Actor);
					}

					if (Actor.IsValid())
					{
						UpdateToolTip();
					}
				}
			}
			else
			{
				Actor.Reset();
				UpdateToolTip();
			}
		}
	}
}

void FSourceControlViewportToolTips::UpdateToolTip()
{
	ToolTipText = FText::GetEmpty();

	if (Actor.IsValid())
	{
		bool bExternal = Actor->IsPackageExternal();
		bool bIgnored = !bExternal;
		bool bSelected = Actor->IsSelected();
		bool bHidden = Actor->IsHidden();
		if (!bIgnored && !bSelected && !bHidden)
		{
			if (UPackage* Package = Actor->GetPackage())
			{
				FString SourceFileName = SourceControlHelpers::PackageFilename(Package);

				ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
				ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
				if (SourceControlModule.IsEnabled())
				{
					FSourceControlStatePtr State = SourceControlProvider.GetState(SourceFileName, EStateCacheUsage::Use);
					if (State.IsValid() && State->IsSourceControlled())
					{
						FString Who;

						bool bNotAtHeadRevision = !State->IsCurrent();
						bool bCheckedOutByOtherUser = State->IsCheckedOutOther(&Who);
						bool bCheckedOut = State->IsCheckedOut();
						bool bOpenForAdd = State->IsAdded();

						bool bNotAtHeadRevisionEnabled = false;
						bool bCheckedOutByOtherUserEnabled = false;
						bool bCheckedOutEnabled = false;
						bool bOpenForAddEnabled = false;

						if (TSharedPtr<SLevelViewport> ViewportWidgetPtr = ViewportWidget.Pin())
						{
							bNotAtHeadRevisionEnabled = SourceControlViewportUtils::GetFeedbackEnabled(ViewportWidgetPtr->GetViewportClient().Get(), ESourceControlStatus::NotAtHeadRevision);
							bCheckedOutByOtherUserEnabled = SourceControlViewportUtils::GetFeedbackEnabled(ViewportWidgetPtr->GetViewportClient().Get(), ESourceControlStatus::CheckedOutByOtherUser);
							bCheckedOutEnabled = SourceControlViewportUtils::GetFeedbackEnabled(ViewportWidgetPtr->GetViewportClient().Get(), ESourceControlStatus::CheckedOut);
							bOpenForAddEnabled = SourceControlViewportUtils::GetFeedbackEnabled(ViewportWidgetPtr->GetViewportClient().Get(), ESourceControlStatus::OpenForAdd);
						}

						if (bNotAtHeadRevision && bNotAtHeadRevisionEnabled)
						{
							ToolTipText = LOCTEXT("NotAtHeadRevision", "File(s) out of sync");
						}
						else if (bCheckedOutByOtherUser && bCheckedOutByOtherUserEnabled)
						{
							ToolTipText = FText::Format(LOCTEXT("CheckedOutOtherUser", "File(s) checked out by {0}"), FText::FromString(Who));
						}
						else if (bCheckedOut && bCheckedOutEnabled)
						{
							ToolTipText = LOCTEXT("CheckedOut", "File(s) checked out by you");
						}
						else if (bOpenForAdd && bOpenForAddEnabled)
						{
							ToolTipText = LOCTEXT("OpenForAdd", "File(s) added by you");
						}
					}
				}
			}
		}
	}

	ToolTipWidget->SetVisibility(ToolTipText.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible);
}

void FSourceControlViewportToolTips::InsertCanvas()
{
	check(!CanvasWidget.IsValid());
	check(!ToolTipWidget.IsValid());
	check(!ViewportWidget.IsValid());

	if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->IsPerspective())
	{
		TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(GCurrentLevelEditingViewportClient->GetEditorViewportWidget());
		if (LevelViewport.IsValid())
		{
			CanvasWidget = SNew(SCanvas)
			+SCanvas::Slot()
				.Position(this, &FSourceControlViewportToolTips::GetToolTipPosition)
				.Size(this, &FSourceControlViewportToolTips::GetToolTipSize)
			[
				SAssignNew(ToolTipWidget, SToolTip)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.Font"))
				.Text_Lambda([this]() { return ToolTipText; })
				.Visibility(EVisibility::Hidden)
			];

			ViewportWidget = LevelViewport;
			ViewportWidget.Pin()->AddOverlayWidget(CanvasWidget.ToSharedRef());
		}
	}
}

void FSourceControlViewportToolTips::RemoveCanvas()
{
	if (TSharedPtr<SLevelViewport> ViewportWidgetPtr = ViewportWidget.Pin())
	{
		ViewportWidgetPtr->RemoveOverlayWidget(CanvasWidget.ToSharedRef());
	}

	CanvasWidget.Reset();
	ToolTipWidget.Reset();
	ViewportWidget.Reset();
}

FVector2D FSourceControlViewportToolTips::GetToolTipPosition() const
{
	FVector2D Result = FVector2D::Zero();

	// Taken from SlateUser.cpp
	static const FVector2f TooltipOffsetFromMouse(12.0f, 8.0f);
	static const FVector2f TooltipOffsetFromForceField(4.0f, 3.0f);

	if (TSharedPtr<SLevelViewport> ViewportWidgetPtr = ViewportWidget.Pin())
	{
		FIntPoint ViewportOrigin;
		FIntPoint ViewportSize;
		ViewportWidgetPtr->GetViewportClient()->GetViewportDimensions(ViewportOrigin, ViewportSize);

		int32 MouseX = 0;
		int32 MouseY = 0;
		if (FViewport* Viewport = ViewportWidgetPtr->GetViewportClient()->Viewport)
		{
			MouseX = Viewport->GetMouseX();
			MouseY = Viewport->GetMouseY();
		}

		FVector2D ToolTipPos(MouseX + TooltipOffsetFromMouse.X, MouseY + TooltipOffsetFromMouse.Y);
		FVector2D ToolTipSize = GetToolTipSize();
	
		if (ToolTipPos.X + ToolTipSize.X + TooltipOffsetFromForceField.X > ViewportSize.X)
		{
			// Flip tooltip left so we don't overflow on the right.
			ToolTipPos.X = MouseX - TooltipOffsetFromMouse.X - ToolTipSize.X;
		}
		if (ToolTipPos.Y + ToolTipSize.Y + TooltipOffsetFromForceField.Y > ViewportSize.Y)
		{
			// Flip tooltip up so we don't overflow on the bottom.
			ToolTipPos.Y = MouseY - TooltipOffsetFromMouse.Y - ToolTipSize.Y;
		}

		Result = ToolTipPos;
	}

	return Result;
}

FVector2D FSourceControlViewportToolTips::GetToolTipSize() const
{
	FVector2D Result = FVector2D::Zero();
	
	if (ToolTipWidget.IsValid())
	{
		FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.Font");

		FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(ToolTipText, FontInfo);
		FVector2D TextMargin = FVector2D(2 * 11.f, 2 * 11.f);

		Result = TextSize + TextMargin;
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE