// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class SLevelViewport;
class SCanvas;
class SToolTip;

// Adds a tooltip widget to the viewport.
class FSourceControlViewportToolTips : public TSharedFromThis<FSourceControlViewportToolTips, ESPMode::ThreadSafe>
{
public:
	FSourceControlViewportToolTips();
	~FSourceControlViewportToolTips();

public:
	void Init();
	void SetEnabled(bool bInEnabled);

	bool Tick(float DeltaTime);

private:
	void UpdateCanvas(float DeltaTime);
	void UpdateToolTip();

	void InsertCanvas();
	void RemoveCanvas();

	FVector2D GetToolTipSize() const;
	FVector2D GetToolTipPosition() const;

private:
	/* The ticker handle */
	FTSTicker::FDelegateHandle TickHandle;

	/* The canvas widget */
	TSharedPtr<SCanvas> CanvasWidget;

	/* The tooltip widget that'll be positioned on the canvas */
	TSharedPtr<SToolTip> ToolTipWidget;

	/* The viewport in which the canvas widget is added */
	TWeakPtr<SLevelViewport> ViewportWidget;

	/* The actor in the viewport that's currently hovered */
	TWeakObjectPtr<AActor> Actor;

	/* The tooltip text determined for the actor */
	FText ToolTipText;

	/* The previous mouse coordinates */
	int32 ActorMouseX = 0;
	int32 ActorMouseY = 0;

	/* The time since the actor became relevant */
	float DelayTime = 0;

	/* The enabled state of the tooltips */
	bool bEnabled = false;
};