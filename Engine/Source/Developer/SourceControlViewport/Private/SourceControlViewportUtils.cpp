// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlViewportUtils.h"
#include "RevisionControlOverlaySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceControlViewportUtils)

namespace SourceControlViewportUtils
{

namespace Private
{
	bool* FindSettingForStatus(URevisionControlOverlaySettings* Settings, ESourceControlStatus Status)
	{
		if (!Settings)
		{
			return nullptr;
		}

		switch (Status)
		{
		case ESourceControlStatus::CheckedOutByOtherUser:
			return &Settings->bShowCheckedOutByOtherUser;
		case ESourceControlStatus::NotAtHeadRevision:
			return &Settings->bShowNotAtHeadRevision;
		case ESourceControlStatus::CheckedOut:
			return &Settings->bShowCheckedOut;
		case ESourceControlStatus::OpenForAdd:
			return &Settings->bShowOpenForAdd;
		}
		checkNoEntry();
		return nullptr;
	}
}

bool GetFeedbackEnabled(FViewportClient* ViewportClient, ESourceControlStatus Status)
{
	const URevisionControlOverlaySettings* Settings = GetDefault<URevisionControlOverlaySettings>();
	if (const bool* Value = Private::FindSettingForStatus(const_cast<URevisionControlOverlaySettings*>(Settings), Status))
	{
		return *Value;
	}
	return false;
}

void SetFeedbackEnabled(FViewportClient* ViewportClient, ESourceControlStatus Status, bool bEnabled)
{
	URevisionControlOverlaySettings* Settings = GetMutableDefault<URevisionControlOverlaySettings>();
	if (bool* Value = Private::FindSettingForStatus(Settings, Status))
	{
		if (*Value != bEnabled)
		{
			*Value = bEnabled;
			Settings->SaveConfig();
			URevisionControlOverlaySettings::NotifyOverlayStatesChanged();
		}
	}
}

uint8 GetFeedbackOpacity(FViewportClient* ViewportClient)
{
	const URevisionControlOverlaySettings* Settings = GetDefault<URevisionControlOverlaySettings>();
	return static_cast<uint8>(FMath::Clamp(Settings->OverlayAlpha, 0, 100));
}

void SetFeedbackOpacity(FViewportClient* ViewportClient, uint8 Opacity)
{
	URevisionControlOverlaySettings* Settings = GetMutableDefault<URevisionControlOverlaySettings>();
	const int32 Clamped = FMath::Clamp(static_cast<int32>(Opacity), 0, 100);
	if (Settings->OverlayAlpha != Clamped)
	{
		Settings->OverlayAlpha = Clamped;
		URevisionControlOverlaySettings::NotifyOverlayColorsChanged();
	}
}

void SaveFeedbackSettings()
{
	GetMutableDefault<URevisionControlOverlaySettings>()->SaveConfig();
}

}
