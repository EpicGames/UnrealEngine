// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "SourceControlViewportUtils.generated.h"

class FViewportClient;

UENUM()
enum class ESourceControlStatus : uint8
{
	/* The file is checked out by another user */
	CheckedOutByOtherUser,

	/* The file is not at the latest revision */
	NotAtHeadRevision,

	/* The file is checked out by self */
	CheckedOut,

	/* The file is added by self */
	OpenForAdd,
};

namespace SourceControlViewportUtils
{

// Helper method to get the viewport feedback value from the viewport.
bool GetFeedbackEnabled(FViewportClient* ViewportClient, ESourceControlStatus Status);

// Helper method to set the viewport feedback on or off on the viewport.
void SetFeedbackEnabled(FViewportClient* ViewportClient, ESourceControlStatus Status, bool bEnabled);

// Helper method to get the opacity value.
uint8 GetFeedbackOpacity(FViewportClient* ViewportClient);

// Helper method to set the opacity value. Refreshes the viewport but does not persist; call SaveFeedbackSettings when the user commits.
void SetFeedbackOpacity(FViewportClient* ViewportClient, uint8 Opacity);

// Persist current overlay settings to ini.
void SaveFeedbackSettings();

}