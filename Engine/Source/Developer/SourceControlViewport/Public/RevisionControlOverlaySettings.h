// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Engine/DeveloperSettings.h"
#include "RevisionControlOverlaySettings.generated.h"

/**
 * User preferences for the viewport revision control overlays.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, defaultconfig, meta=(DisplayName="Revision Control Overlays"))
class URevisionControlOverlaySettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	virtual FName GetCategoryName() const override { return TEXT("ContentEditors"); }

	UPROPERTY(config, EditAnywhere, Category="Viewport Overlays")
	bool bShowCheckedOutByOtherUser = true;

	UPROPERTY(config, EditAnywhere, Category="Viewport Overlays")
	bool bShowNotAtHeadRevision = true;

	UPROPERTY(config, EditAnywhere, Category="Viewport Overlays")
	bool bShowCheckedOut = false;

	UPROPERTY(config, EditAnywhere, Category="Viewport Overlays")
	bool bShowOpenForAdd = false;

	UPROPERTY(config, EditAnywhere, Category="Viewport Overlays", meta=(ClampMin=0, ClampMax=100, UIMin=0, UIMax=100))
	int32 OverlayAlpha = 20;

	static SOURCECONTROLVIEWPORT_API FSimpleMulticastDelegate OnOverlayStatesChanged;
	static SOURCECONTROLVIEWPORT_API FSimpleMulticastDelegate OnOverlayColorsChanged;

	static SOURCECONTROLVIEWPORT_API void NotifyOverlayStatesChanged();
	static SOURCECONTROLVIEWPORT_API void NotifyOverlayColorsChanged();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
